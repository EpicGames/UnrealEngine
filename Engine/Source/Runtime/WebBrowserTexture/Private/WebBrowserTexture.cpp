// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserTexture.h"

#include "ExternalTexture.h"
#include "Modules/ModuleManager.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WebBrowserTextureResource.h"
#include "IWebBrowserWindow.h"

/* UWebBrowserTexture structors
*****************************************************************************/

UWebBrowserTexture::UWebBrowserTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AddressX(TA_Clamp)
	, AddressY(TA_Clamp)
	, ClearColor(FLinearColor::Black)
	, Size(0)
{
	SampleQueue = MakeShared<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe>();
	WebPlayerGuid = FGuid::NewGuid();
	NeverStream = true;
}

/* UWebBrowserTexture interface
*****************************************************************************/

float UWebBrowserTexture::GetAspectRatio() const
{
	if (Dimensions.Y == 0)
	{
		return 0.0f;
	}

	return (float)(Dimensions.X) / Dimensions.Y;
}

int32 UWebBrowserTexture::GetHeight() const
{
	return Dimensions.Y;
}

int32 UWebBrowserTexture::GetWidth() const
{
	return Dimensions.X;
}

/* UTexture interface
*****************************************************************************/

FTextureResource* UWebBrowserTexture::CreateResource()
{
	return new FWebBrowserTextureResource(*this, Dimensions, Size);
}

EMaterialValueType UWebBrowserTexture::GetMaterialType() const
{
	return MCT_TextureExternal;
}

float UWebBrowserTexture::GetSurfaceWidth() const
{
	return Dimensions.X;
}

float UWebBrowserTexture::GetSurfaceHeight() const
{
	return Dimensions.Y;
}

FGuid UWebBrowserTexture::GetExternalTextureGuid() const
{
	return WebPlayerGuid;
}

void UWebBrowserTexture::SetExternalTextureGuid(FGuid guid)
{
	WebPlayerGuid = guid;
}

/* UObject interface
*****************************************************************************/

void UWebBrowserTexture::BeginDestroy()
{
	UnregisterPlayerGuid();

	Super::BeginDestroy();
}

FString UWebBrowserTexture::GetDesc()
{
	return FString::Printf(TEXT("%ix%i [%s]"), Dimensions.X, Dimensions.Y, GPixelFormats[PF_B8G8R8A8].Name);
}


void UWebBrowserTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddUnknownMemoryBytes(Size);
}

/* UWebBrowserTexture implementation
*****************************************************************************/


void UWebBrowserTexture::TickResource(TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample)
{
	if (GetResource() == nullptr)
	{
		return;
	}
	
	check(SampleQueue.IsValid());

	if (Sample.IsValid())
	{
		SampleQueue.Get()->Enqueue(Sample);
	}

	// issue a render command to render the current sample
	FWebBrowserTextureResource::FRenderParams RenderParams;
	{
		RenderParams.ClearColor = ClearColor;
		RenderParams.PlayerGuid = GetExternalTextureGuid();
		RenderParams.SampleSource = SampleQueue;
	}

	FWebBrowserTextureResource* ResourceParam = (FWebBrowserTextureResource*)GetResource();
	ENQUEUE_RENDER_COMMAND(UWebBrowserTextureResourceRender)(
		[ResourceParam, RenderParams](FRHICommandListImmediate& RHICmdList)
		{
			ResourceParam->Render(RHICmdList, RenderParams);
		});
}

void UWebBrowserTexture::UnregisterPlayerGuid()
{
	if (!WebPlayerGuid.IsValid())
	{
		return;
	}

	FGuid PlayerGuid = WebPlayerGuid;
	ENQUEUE_RENDER_COMMAND(UWebBrowserTextureUnregisterPlayerGuid)(
		[PlayerGuid](FRHICommandListImmediate& RHICmdList)
		{
			FExternalTextureRegistry::Get().UnregisterExternalTexture(PlayerGuid);
		});
}
