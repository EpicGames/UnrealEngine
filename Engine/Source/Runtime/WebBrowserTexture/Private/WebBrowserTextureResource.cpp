// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserTextureResource.h"

#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "ExternalTexture.h"
#include "PipelineStateCache.h"
#include "SceneUtils.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "RenderUtils.h"
#include "RHIStaticStates.h"
#include "ExternalTexture.h"
#include "WebBrowserTexture.h"


#define WebBrowserTextureRESOURCE_TRACE_RENDER 0

DEFINE_LOG_CATEGORY(LogWebBrowserTexture);

/* FWebBrowserTextureResource structors
 *****************************************************************************/

FWebBrowserTextureResource::FWebBrowserTextureResource(UWebBrowserTexture& InOwner, FIntPoint& InOwnerDim, SIZE_T& InOwnerSize)
	: Cleared(false)
	, CurrentClearColor(FLinearColor::Transparent)
	, Owner(InOwner)
	, OwnerDim(InOwnerDim)
	, OwnerSize(InOwnerSize)
{
	UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:FWebBrowserTextureResource %d %d", OwnerDim.X, OwnerDim.Y);
}


/* FWebBrowserTextureResource interface
 *****************************************************************************/

void FWebBrowserTextureResource::Render(FRHICommandList& RHICmdList, const FRenderParams& Params)
{
	check(IsInRenderingThread());

	TSharedPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> SampleSource = Params.SampleSource.Pin();

	if (SampleSource.IsValid())
	{
		// get the most current sample to be rendered
		TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample;
		bool SampleValid = false;
		
		while (SampleSource->Peek(Sample) && Sample.IsValid())
		{
			SampleValid = SampleSource->Dequeue(Sample);
		}

		if (!SampleValid)
		{
			return; // no sample to render
		}

		check(Sample.IsValid());

		// render the sample
		CopySample(RHICmdList, Sample, Params.ClearColor);

		if (!GSupportsImageExternal && Params.PlayerGuid.IsValid())
		{
			FTextureRHIRef VideoTexture = (FTextureRHIRef)Owner.TextureReference.TextureReferenceRHI;
			FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, VideoTexture, SamplerStateRHI, Sample->GetScaleRotation(), Sample->GetOffset());
		}
	}
	else if (!Cleared)
	{
		ClearTexture(RHICmdList, Params.ClearColor);

		if (!GSupportsImageExternal && Params.PlayerGuid.IsValid())
		{
			FTextureRHIRef VideoTexture = (FTextureRHIRef)Owner.TextureReference.TextureReferenceRHI;
			FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, VideoTexture, SamplerStateRHI, FLinearColor(1.0f, 0.0f, 0.0f, 1.0f), FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
		}
	}
}


/* FRenderTarget interface
 *****************************************************************************/

FIntPoint FWebBrowserTextureResource::GetSizeXY() const
{
	return FIntPoint(Owner.GetWidth(), Owner.GetHeight());
}


/* FTextureResource interface
 *****************************************************************************/

FString FWebBrowserTextureResource::GetFriendlyName() const
{
	return Owner.GetPathName();
}


uint32 FWebBrowserTextureResource::GetSizeX() const
{
	return Owner.GetWidth();
}


uint32 FWebBrowserTextureResource::GetSizeY() const
{
	return Owner.GetHeight();
}


void FWebBrowserTextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create the sampler state
	FSamplerStateInitializerRHI SamplerStateInitializer(
		(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(&Owner),
		(Owner.AddressX == TA_Wrap) ? AM_Wrap : ((Owner.AddressX == TA_Clamp) ? AM_Clamp : AM_Mirror),
		(Owner.AddressY == TA_Wrap) ? AM_Wrap : ((Owner.AddressY == TA_Clamp) ? AM_Clamp : AM_Mirror),
		AM_Wrap
	);

	SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
}


void FWebBrowserTextureResource::ReleaseRHI()
{
	Cleared = false;

	InputTarget.SafeRelease();
	OutputTarget.SafeRelease();
	RenderTargetTextureRHI.SafeRelease();
	TextureRHI.SafeRelease();

	RHIClearTextureReference(Owner.TextureReference.TextureReferenceRHI);
	OwnerDim = FIntPoint::ZeroValue;
}


/* FWebBrowserTextureResource implementation
 *****************************************************************************/

void FWebBrowserTextureResource::ClearTexture(FRHICommandList& RHICmdList, const FLinearColor& ClearColor)
{
	UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:ClearTexture");
	// create output render target if we don't have one yet
	const ETextureCreateFlags OutputCreateFlags = ETextureCreateFlags::SRGB;

	if ((ClearColor != CurrentClearColor) || !OutputTarget.IsValid() || !EnumHasAllFlags(OutputTarget->GetFlags(), OutputCreateFlags))
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FWebBrowserTextureResource"))
			.SetExtent(2, 2)
			.SetFormat(PF_B8G8R8A8)
			.SetFlags(OutputCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClearValue(FClearValueBinding(ClearColor));

		OutputTarget = RHICmdList.CreateTexture(Desc);

		CurrentClearColor = ClearColor;
		UpdateResourceSize();
	}

	if (RenderTargetTextureRHI != OutputTarget)
	{
		UpdateTextureReference(RHICmdList, OutputTarget);
	}

	// draw the clear color
	{
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
		ClearRenderTarget(RHICmdList, RenderTargetTextureRHI);
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}

	Cleared = true;
}

void FWebBrowserTextureResource::CopySample(FRHICommandListBase& RHICmdList, const TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor)
{
	FRHITexture* SampleTexture = Sample->GetTexture();
	FRHITexture* SampleTexture2D = (SampleTexture != nullptr) ? SampleTexture->GetTexture2D() : nullptr;
	// If the sample already provides a texture resource, we simply use that
	// as the output render target. If the sample only provides raw data, then
	// we create our own output render target and copy the data into it.
	if (SampleTexture2D != nullptr)
	{
		UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:CopySample 1");
		// use sample's texture as the new render target.
		if (TextureRHI != SampleTexture2D)
		{
			UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:CopySample 11");
			UpdateTextureReference(RHICmdList, SampleTexture2D);

			OutputTarget.SafeRelease();
			UpdateResourceSize();
		}
	}
	else
	{
		UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:CopySample 2");
		// create a new output render target if necessary
		const ETextureCreateFlags OutputCreateFlags = TexCreate_SRGB;
		const FIntPoint SampleDim = Sample->GetDim();

		if ((ClearColor != CurrentClearColor) || !OutputTarget.IsValid() || (OutputTarget->GetSizeXY() != SampleDim) || ((OutputTarget->GetFlags() & OutputCreateFlags) != OutputCreateFlags))
		{
			UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:CopySample 1");

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FWebBrowserTextureResource"))
				.SetExtent(SampleDim)
				.SetFormat(PF_B8G8R8A8)
				.SetFlags(OutputCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetInitialState(ERHIAccess::SRVMask)
				.SetClearValue(FClearValueBinding(ClearColor));

			OutputTarget = RHICmdList.CreateTexture(Desc);

			CurrentClearColor = ClearColor;
			UpdateResourceSize();
		}

		if (RenderTargetTextureRHI != OutputTarget)
		{
			UpdateTextureReference(RHICmdList, OutputTarget);
		}

		UE_LOGF(LogWebBrowserTexture, VeryVerbose, "WebBrowserTextureResource:CopySample: %d x %d", SampleDim.X, SampleDim.Y);

		// copy sample data to output render target
		FUpdateTextureRegion2D Region(0, 0, 0, 0, SampleDim.X, SampleDim.Y);
		RHICmdList.UpdateTexture2D(RenderTargetTextureRHI.GetReference(), 0, Region, Sample->GetStride(), (uint8*)Sample->GetBuffer());
	}
	Cleared = false;
}


void FWebBrowserTextureResource::UpdateResourceSize()
{
	UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:UpdateResourceSize");

	SIZE_T ResourceSize = 0;

	if (InputTarget.IsValid())
	{
		ResourceSize += CalcTextureSize(InputTarget->GetSizeX(), InputTarget->GetSizeY(), InputTarget->GetFormat(), 1);
	}

	if (OutputTarget.IsValid())
	{
		ResourceSize += CalcTextureSize(OutputTarget->GetSizeX(), OutputTarget->GetSizeY(), OutputTarget->GetFormat(), 1);
	}

	OwnerSize = ResourceSize;
}


void FWebBrowserTextureResource::UpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* NewTexture)
{
	TextureRHI = NewTexture;
	RenderTargetTextureRHI = NewTexture;

	RHICmdList.UpdateTextureReference(Owner.TextureReference.TextureReferenceRHI, NewTexture);

	if (RenderTargetTextureRHI != nullptr)
	{
		OwnerDim = FIntPoint(RenderTargetTextureRHI->GetSizeX(), RenderTargetTextureRHI->GetSizeY());
	}
	else
	{
		OwnerDim = FIntPoint::ZeroValue;
	}
	UE_LOGF(LogWebBrowserTexture, VeryVerbose, "FWebBrowserTextureResource:UpdateTextureReference: %d x %d", OwnerDim.X, OwnerDim.Y);
}
