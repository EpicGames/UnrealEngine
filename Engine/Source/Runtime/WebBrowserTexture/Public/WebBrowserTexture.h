// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/EnumAsByte.h"
#include "Engine/Texture.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "WebBrowserTextureSample.h"

#include "WebBrowserTexture.generated.h"

/**
* Implements a texture asset for rendering webbrowser output for Android.
* 
*  probably should have derived from UTexture2DDynamic (not UTexture)
*/
UCLASS(hidecategories = (Adjustments, Compositing, LevelOfDetail, Object))
class WEBBROWSERTEXTURE_API UWebBrowserTexture
	: public UTexture
{
	GENERATED_UCLASS_BODY()

	/** The addressing mode to use for the X axis. */
	TEnumAsByte<TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis. */
	TEnumAsByte<TextureAddress> AddressY;

	/** Whether to clear the texture when no media is being played (default = enabled). */
	bool AutoClear;

	/** The color used to clear the texture if AutoClear is enabled (default = black). */
	FLinearColor ClearColor;

public:

	/**
	* Gets the current aspect ratio of the texture.
	*
	* @return Texture aspect ratio.
	* @see GetHeight, GetWidth
	*/
	float GetAspectRatio() const;

	/**
	* Gets the current height of the texture.
	*
	* @return Texture height (in pixels).
	* @see GetAspectRatio, GetWidth
	*/
	int32 GetHeight() const;

	/**
	* Gets the current width of the texture.
	*
	* @return Texture width (in pixels).
	* @see GetAspectRatio, GetHeight
	*/
	int32 GetWidth() const;

public:
	//~ UTexture interface.
	virtual void BeginDestroy() override;
	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override;
	virtual float GetSurfaceWidth() const override;
	virtual float GetSurfaceHeight() const override;
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual FGuid GetExternalTextureGuid() const override;
	virtual ETextureClass GetTextureClass() const { return ETextureClass::Other2DNoSource; }

public:
	//~ UObject interface.
	virtual FString GetDesc() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	
	void TickResource(TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample);

	void SetExternalTextureGuid(FGuid guid);
protected:
	/** Unregister the player's external texture GUID. */
	void UnregisterPlayerGuid();

private:
	/** Current width and height of the resource (in pixels). */
	FIntPoint Dimensions;

	/** The previously used player GUID. */
	FGuid WebPlayerGuid;

	/** Texture sample queue. */
	TSharedPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> SampleQueue;

	/** Current size of the resource (in bytes).*/
	SIZE_T Size;
};
