// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * WebBrowserTextureModule interface
 */
class IWebBrowserTextureModule : public IModuleInterface
{
};
