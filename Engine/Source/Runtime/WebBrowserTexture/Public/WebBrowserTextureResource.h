// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Queue.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "TextureResource.h"
#include "UnrealClient.h"
#include "WebBrowserTextureSample.h"

class UWebBrowserTexture;

WEBBROWSERTEXTURE_API DECLARE_LOG_CATEGORY_EXTERN(LogWebBrowserTexture, Log, All);

/**
 * Texture resource type for webbrowser textures.
 */
class FWebBrowserTextureResource
	: public FRenderTarget
	, public FTextureResource
{
public:

	/** 
	 * Creates and initializes a new instance.
	 *
	 * @param InOwner The Movie texture object to create a resource for (must not be nullptr).
	 * @param InOwnerDim The initial width and height of the texture.
	 * @param InOwnerSize The initial memory allocated to the texture.
	 */
	FWebBrowserTextureResource(UWebBrowserTexture& InOwner, FIntPoint& InOwnerDim, SIZE_T& InOwnerSize);

	/** Virtual destructor. */
	virtual ~FWebBrowserTextureResource() { }

public:

	/** Parameters for the Render method. */
	struct FRenderParams
	{
		/** The clear color to use when clearing the texture. */
		FLinearColor ClearColor;

		/** Guid associated with the texture. */
		FGuid PlayerGuid;

		/** The sample source to render. */
		TWeakPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> SampleSource;

	};

	/**
	 * Render the texture resource.
	 *
	 * This method is called on the render thread by the WebBrowserTexture that owns this
	 * texture resource to clear or redraw the resource using the given parameters.
	 *
	 * @param Params Render parameters.
	 */
	void Render(FRHICommandList& RHICmdList, const FRenderParams& Params);

public:

	//~ FRenderTarget interface

	virtual FIntPoint GetSizeXY() const override;

public:

	//~ FTextureResource interface

	virtual FString GetFriendlyName() const override;
	virtual uint32 GetSizeX() const override;
	virtual uint32 GetSizeY() const override;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

protected:

	/**
	 * Clear the texture using the given clear color.
	 *
	 * @param ClearColor The clear color to use.
	 */
	void ClearTexture(FRHICommandList& RHICmdList, const FLinearColor& ClearColor);

	/**
	 * Render the given texture sample by using it as or copying it to the render target.
	 *
	 * @param Sample The texture sample to copy.
	 * @param ClearColor The clear color to use for the output texture.
	 * @see ConvertSample
	 */
	void CopySample(FRHICommandListBase& RHICmdList, const TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>& Sample, const FLinearColor& ClearColor);

	/** Calculates the current resource size and notifies the owner texture. */
	void UpdateResourceSize();

	/**
	 * Set the owner's texture reference to the given texture.
	 *
	 * @param NewTexture The texture to set.
	 */
	void UpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* NewTexture);

private:

	/** Whether the texture has been cleared. */
	bool Cleared;

	/** Tracks the current clear color. */
	FLinearColor CurrentClearColor;

	/** Input render target if the texture samples don't provide one (for conversions). */
	TRefCountPtr<FRHITexture> InputTarget;

	/** Output render target if the texture samples don't provide one. */
	TRefCountPtr<FRHITexture> OutputTarget;

	/** The webbrowser texture that owns this resource. */
	UWebBrowserTexture& Owner;

	/** Reference to the owner's texture dimensions field. */
	FIntPoint& OwnerDim;

	/** Reference to the owner's texture size field. */
	SIZE_T& OwnerSize;
};
