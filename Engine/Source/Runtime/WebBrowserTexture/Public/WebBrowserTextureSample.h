// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreGlobals.h"
#include "Containers/Queue.h"
#include "Math/IntPoint.h"
#include "Math/Range.h"
#include "MediaObjectPool.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "Templates/SharedPointer.h"

/**
* Texture sample generated by the WebBrowser.
*/
class FWebBrowserTextureSample
	: public IMediaPoolable
{
public:

	/** Default constructor. */
	FWebBrowserTextureSample()
		: Buffer(nullptr)
		, BufferSize(0)
		, Dim(FIntPoint::ZeroValue)
		, ScaleRotation(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f))
		, Offset(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
	{ }

	/** Virtual destructor. */
	virtual ~FWebBrowserTextureSample()
	{
		if (BufferSize > 0)
		{
			FMemory::Free(Buffer);
		}
	}

public:

	/**
	* Get a writable pointer to the sample buffer.
	*
	* @return Sample buffer.
	*/
	void* GetMutableBuffer()
	{
		return Buffer;
	}

	/**
	* Initialize the sample.
	*
	* @param InDim The sample buffer's width and height (in pixels).
	* @param InDuration The duration for which the sample is valid.
	* @return true on success, false otherwise.
	*/
	bool Initialize(const FIntPoint& InDim)
	{
		if (InDim.GetMin() <= 0)
		{ 
			return false;
		}

		Dim = InDim;

		return true;
	}

	/**
	* Initialize the sample with a memory buffer.
	*
	* @param InBuffer The buffer containing the sample data.
	* @param Copy Whether the buffer should be copied (true) or referenced (false).
	* @see InitializeTexture
	*/
	void InitializeBuffer(void* InBuffer, bool Copy)
	{
		if (Copy)
		{
			SIZE_T RequiredBufferSize = Dim.X * Dim.Y * sizeof(int32);
			if (BufferSize < RequiredBufferSize)
			{
				if (BufferSize == 0)
				{
					Buffer = FMemory::Malloc(RequiredBufferSize);
				}
				else
				{
					Buffer = FMemory::Realloc(Buffer, RequiredBufferSize);
				}

				BufferSize = RequiredBufferSize;
			}

			FMemory::Memcpy(Buffer, InBuffer, RequiredBufferSize);
		}
		else
		{
			if (BufferSize > 0)
			{
				FMemory::Free(Buffer);
				BufferSize = 0;
			}

			Buffer = InBuffer;
		}
	}

	/**
	* Initialize the sample for copy externally.
	*
	* @return The size of the buffer
	* @see InitializeTexture
	*/
	int32 InitializeBufferForCopy()
	{
		SIZE_T RequiredBufferSize = Dim.X * Dim.Y * sizeof(int32);
		if (BufferSize < RequiredBufferSize)
		{
			if (BufferSize == 0)
			{
				Buffer = FMemory::Malloc(RequiredBufferSize);
			}
			else
			{
				Buffer = FMemory::Realloc(Buffer, RequiredBufferSize);
			}

			BufferSize = RequiredBufferSize;
		}
		return BufferSize;
	}

	/**
	* Initialize the sample with a texture resource.
	*
	* @param RHICmdList the command list used to create the texture.
	* @return The texture resource object that will hold the sample data.
	* @note This method must be called on the render thread.
	* @see InitializeBuffer
	*/
	FRHITexture* InitializeTexture(FRHICommandListBase& RHICmdList)
	{
		check(IsInRenderingThread());

		if (Texture.IsValid() && (Texture->GetSizeXY() == Dim))
		{
			return Texture;
		}

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FWebBrowserTextureSample"))
			.SetExtent(Dim)
			.SetFormat(PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::SRGB | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask);

		Texture = RHICmdList.CreateTexture(Desc);

		return Texture;
	}

	/**
	* Set the sample Scale, Rotation, Offset.
	*
	* @param InScaleRotation The sample scale and rotation transform (2x2).
	* @param InOffset The sample offset.
	*/
	void SetScaleRotationOffset(FVector4& InScaleRotation, FVector4& InOffset)
	{
		ScaleRotation = FLinearColor(InScaleRotation.X, InScaleRotation.Y, InScaleRotation.Z, InScaleRotation.W);
		Offset = FLinearColor(InOffset.X, InOffset.Y, InOffset.Z, InOffset.W);
	}

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() 
	{
		return Buffer;
	}

	virtual FIntPoint GetDim() const
	{
		return Dim;
	}

	virtual uint32 GetStride() const
	{
		return Dim.X * sizeof(int32);
	}

#if WITH_ENGINE

	virtual FRHITexture* GetTexture() const
	{
		return Texture.GetReference();
	}

#endif //WITH_ENGINE

	virtual FLinearColor GetScaleRotation() const
	{
		return ScaleRotation;
	}

	virtual FLinearColor GetOffset() const
	{
		return Offset;
	}

private:

	/** The sample's data buffer. */
	void* Buffer;

	/** Current allocation size of Buffer. */
	SIZE_T BufferSize;

	/** Width and height of the texture sample. */
	FIntPoint Dim;

	/** ScaleRotation for the sample. */
	FLinearColor ScaleRotation;

	/** Offset for the sample. */
	FLinearColor Offset;

#if WITH_ENGINE

	/** Texture resource. */
	TRefCountPtr<FRHITexture> Texture;

#endif //WITH_ENGINE
};


class FWebBrowserTextureSampleQueue
{
public:

	/** Default constructor. */
	FWebBrowserTextureSampleQueue()
		: NumSamples(0)
		, PendingFlushes(0)
	{ }

	/** Virtual destructor. */
	virtual ~FWebBrowserTextureSampleQueue() { }

public:

	/**
	* Get the number of samples in the queue.
	*
	* @return Number of samples.
	* @see Enqueue, Dequeue, Peek
	*/
	int32 Num() const
	{
		return NumSamples;
	}

public:

	//~ TMediaSampleSource interface (to be called only from consumer thread)

	virtual bool Dequeue(TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>& OutSample)
	{
		DoPendingFlushes();

		TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample;

		if (!Samples.Peek(Sample))
		{
			return false; // empty queue
		}

		if (!Sample.IsValid())
		{
			return false; // pending flush
		}

		Samples.Pop();

		FPlatformAtomics::InterlockedDecrement(&NumSamples);
		check(NumSamples >= 0);

		OutSample = Sample;

		return true;
	}

	virtual bool Peek(TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>& OutSample)
	{
		DoPendingFlushes();

		TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample;

		if (!Samples.Peek(Sample))
		{
			return false; // empty queue
		}

		if (!Sample.IsValid())
		{
			return false; // pending flush
		}

		OutSample = Sample;

		return true;
	}

	virtual bool Pop()
	{
		TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample;

		if (!Samples.Peek(Sample))
		{
			return false; // empty queue
		}

		if (!Sample.IsValid())
		{
			return false; // pending flush
		}

		Samples.Pop();

		FPlatformAtomics::InterlockedDecrement(&NumSamples);
		check(NumSamples >= 0);

		return true;
	}

public:

	//~ TMediaSampleSink interface (to be called only from producer thread)

	virtual bool Enqueue(const TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		if (Sample.IsValid())
		{
			FPlatformAtomics::InterlockedIncrement(&NumSamples);
		}

		if (!Samples.Enqueue(Sample))
		{
			if (Sample.IsValid())
			{
				FPlatformAtomics::InterlockedDecrement(&NumSamples);
			}

			return false;
		}

		return true;
	}

	virtual void RequestFlush()
	{
		Samples.Enqueue(nullptr); // insert flush marker
		FPlatformAtomics::InterlockedIncrement(&PendingFlushes);
	}

protected:

	/** Perform any pending flushes. */
	void DoPendingFlushes()
	{
		TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> Sample;

		while ((PendingFlushes > 0) && Samples.Dequeue(Sample))
		{
			if (Sample.IsValid())
			{
				FPlatformAtomics::InterlockedDecrement(&NumSamples);
				check(NumSamples >= 0);
			}
			else
			{
				FPlatformAtomics::InterlockedDecrement(&PendingFlushes);
			}
		}
	}

private:

	/** Number of samples in the queue. */
	int32 NumSamples;

	/** Number of pending flushes. */
	int32 PendingFlushes;

	/** Audio sample queue. */
	TQueue<TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe>, EQueueMode::Mpsc> Samples;
};


/** Implements a pool for WebBrowser's texture sample objects. */
class FWebBrowserTextureSamplePool : public TMediaObjectPool<FWebBrowserTextureSample> { };

