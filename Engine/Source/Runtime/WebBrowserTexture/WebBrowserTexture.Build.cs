// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class WebBrowserTexture : ModuleRules
{
	public WebBrowserTexture(ReadOnlyTargetRules Target) : base(Target)
	{
        // WebBrowserTexture objects are needed only on Android, but we also need to be able to
        // cook the asset so we must include it in editor builds
        if (Target.Platform == UnrealTargetPlatform.Android ||
            Target.IsInPlatformGroup(UnrealPlatformGroup.IOS) ||
            Target.Platform == UnrealTargetPlatform.Mac ||
            Target.bBuildEditor == true)
		{			
			// Needed for external texture support
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"MediaUtils",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ApplicationCore",
					"RHI",
					"InputCore",
					"Slate",
					"SlateCore",
					"Serialization",
					"MediaUtils",
					"RenderCore",
					"Engine",
					"WebBrowser"
				}
			);
		}
		else
		{
			PrecompileForTargets = ModuleRules.PrecompileTargetsType.None;
		}
	}
}
